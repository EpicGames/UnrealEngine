// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPlatformFileSandboxWrapper.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/Guid.h"
#include "Stats/Stats.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "HAL/IPlatformFileModule.h"
#include "Templates/UniquePtr.h"
#include "Async/MappedFileHandle.h"

DEFINE_LOG_CATEGORY(SandboxFile);

#if !defined(PLATFORM_SUPPORTS_DEFAULT_SANDBOX)
	#define PLATFORM_SUPPORTS_DEFAULT_SANDBOX PLATFORM_DESKTOP
#endif

#if PLATFORM_SUPPORTS_DEFAULT_SANDBOX && (UE_GAME || UE_SERVER)
static FString GetCookedSandboxDir()
{
	TStringBuilder<512> Dir;
	Dir << *FPaths::ProjectDir();
	Dir << "Saved/Cooked/";
	Dir << FPlatformProperties::PlatformName();
	Dir << "/";
	return Dir.ToString();
}
#endif

//////////////////////////////////////////////////////////////////////////

TUniquePtr<FSandboxPlatformFile> FSandboxPlatformFile::Create(bool bInEntireEngineWillUseThisSandbox)
{
	return TUniquePtr<FSandboxPlatformFile>(new FSandboxPlatformFile(bInEntireEngineWillUseThisSandbox));
}

FSandboxPlatformFile::FSandboxPlatformFile(bool bInEntireEngineWillUseThisSandbox)
	: LowerLevel(nullptr)
	, bEntireEngineWillUseThisSandbox(bInEntireEngineWillUseThisSandbox)
	, bSandboxEnabled(true)
	, bSandboxOnly(false)
{
}

FSandboxPlatformFile::~FSandboxPlatformFile()
{
}

bool FSandboxPlatformFile::ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const
{
	FString SandboxDir;
	if (FParse::Value(CmdLine, TEXT("-InjectionSandbox="), SandboxDir))
	{
		return true;
	}
	bool bResult = FParse::Value( CmdLine, TEXT("-Sandbox="), SandboxDir );
#if PLATFORM_SUPPORTS_DEFAULT_SANDBOX && (UE_GAME || UE_SERVER)
	if (FPlatformProperties::RequiresCookedData() && SandboxDir.IsEmpty() && Inner == &FPlatformFileManager::Get().GetPlatformFile() && bEntireEngineWillUseThisSandbox)
	{
		SandboxDir = GetCookedSandboxDir();
		bResult = FPlatformFileManager::Get().GetPlatformFile().DirectoryExists(*SandboxDir);
	}
#endif
	return bResult;
}

bool FSandboxPlatformFile::Initialize(IPlatformFile* Inner, const TCHAR* CmdLine)
{
	FString CommandLineDirectory;

	if (FParse::Value(CmdLine, TEXT("-InjectionSandbox="), CommandLineDirectory))
	{
		int32 DividerLoc = CommandLineDirectory.Find(TEXT(";"));
		checkf(DividerLoc > 0, TEXT("The format is -InjectionSandbox=<RelativeEngineDir>;<AdditionalDir>"));

		InjectedSourceDirectory = CommandLineDirectory.Mid(0, DividerLoc);
		InjectedTargetDirectory = CommandLineDirectory.Mid(DividerLoc + 1);
		InjectedSourceDirectoryParent = FPaths::GetPath(InjectedSourceDirectory);
		InjectedTargetDirectoryParent = FPaths::GetPath(InjectedTargetDirectory);

		// InjectedTargetDirectory is the sandbox directory for the normal logic
		CommandLineDirectory = InjectedTargetDirectory;
	}
	else
	{
		FParse::Value( CmdLine, TEXT("-Sandbox="), CommandLineDirectory);
	}
#if PLATFORM_SUPPORTS_DEFAULT_SANDBOX && (UE_GAME || UE_SERVER)
	if (CommandLineDirectory.IsEmpty() && bEntireEngineWillUseThisSandbox)
	{
		CommandLineDirectory = GetCookedSandboxDir();
		UE_LOGF(LogInit, Display, "No sandbox specified, assuming %ls", *CommandLineDirectory);

		// Don't allow the default cooked sandbox to fallback to non-cooked assets
		FileExclusionWildcards.AddUnique(TEXT("*.uasset"));
		FileExclusionWildcards.AddUnique(TEXT("*.umap"));
	}
#endif

	LowerLevel = Inner;
	if (LowerLevel != NULL && !CommandLineDirectory.IsEmpty())
	{
		// Cache root directory
		RelativeRootDirectory = FPaths::GetRelativePathToRoot();
		AbsoluteRootDirectory = FPaths::ConvertRelativePathToFull(RelativeRootDirectory);

		// Commandline syntax
		bool bWipeSandbox = false;
		FPaths::NormalizeFilename(CommandLineDirectory);
		int32 CommandIndex = CommandLineDirectory.Find(TEXT(":"), ESearchCase::CaseSensitive);
		if( CommandIndex != INDEX_NONE )
		{
			// Check if absolute path was specified and the ':' refers to drive name
			FString DriveCheck( CommandLineDirectory.Mid(0, CommandIndex + 1) );
			if( FPaths::IsDrive(DriveCheck) == false )
			{
				FString Command( CommandLineDirectory.Mid( CommandIndex + 1 ) );
				CommandLineDirectory.LeftInline( CommandIndex, EAllowShrinking::No);
		
				if( Command == TEXT("wipe") )
				{
					bWipeSandbox = true;
				}
				// Add new commands here
			}
		}

		bool bSandboxIsAbsolute = false;
		if( CommandLineDirectory == TEXT("User") )
		{
			// Special case - platform defined user directory will be used
			SandboxDirectory = FPlatformProcess::UserDir();
			SandboxDirectory += TEXT("My Games/");
			SandboxDirectory += TEXT( "UE/" );
			bSandboxIsAbsolute = true;
		}
		else if( CommandLineDirectory == TEXT("Unique") )
		{
			const FString Path = FPaths::GetRelativePathToRoot() / TEXT("");
			SandboxDirectory = FPaths::ConvertToSandboxPath( Path, *FGuid::NewGuid().ToString() );
		}
		else if (CommandLineDirectory.StartsWith(TEXT("..")))
		{
			// for relative-specified directories, just use it directly, and don't put into FPaths::ProjectSavedDir()
			SandboxDirectory = CommandLineDirectory;
		}
		else if( FPaths::IsDrive( CommandLineDirectory.Mid( 0, CommandLineDirectory.Find(TEXT("/"), ESearchCase::CaseSensitive) ) ) == false ) 
		{
			const FString Path = FPaths::GetRelativePathToRoot() / TEXT("");
			SandboxDirectory = FPaths::ConvertToSandboxPath( Path, *CommandLineDirectory );
		}
		else
		{
			SandboxDirectory = CommandLineDirectory;
			bSandboxIsAbsolute = true;
		}
	
		if( !bSandboxIsAbsolute )
		{
			// Make sure all path separators are correct with TEXT("/")
			FPaths::MakeStandardFilename(SandboxDirectory);

			// SandboxDirectory should be absolute and have no relative paths in it
			SandboxDirectory = FPaths::ConvertRelativePathToFull(SandboxDirectory);
		}

		if( bWipeSandbox )
		{
			WipeSandboxFolder( *SandboxDirectory );
		}

		if (SandboxDirectory.EndsWith(TEXT("/")) == false)
		{
			SandboxDirectory += TEXT("/");
		}

		if (bEntireEngineWillUseThisSandbox)
		{
			if (InjectedTargetDirectory.Len())
			{
				FCommandLine::AddToSubprocessCommandLine(*FString::Printf(TEXT("-InjectedSandbox=%s;%s"), *InjectedSourceDirectory, *InjectedTargetDirectory), ECommandLineArgumentFlags::AllContexts);
			}
			else
			{
				FCommandLine::AddToSubprocessCommandLine(*FString::Printf(TEXT("-Sandbox=%s"), *SandboxDirectory), ECommandLineArgumentFlags::AllContexts);
			}
		}
	}

	return !!LowerLevel;
}

const FString& FSandboxPlatformFile::GetSandboxDirectory() const
{
	return SandboxDirectory;
}

const FString& FSandboxPlatformFile::GetAbsoluteRootDirectory() const
{
	return AbsoluteRootDirectory;
}

const FString& FSandboxPlatformFile::GetGameSandboxDirectoryName()
{
	if (GameSandboxDirectoryName.IsEmpty())
	{
		GameSandboxDirectoryName = FString::Printf(TEXT("%s/"), FApp::GetProjectName());
	}
	return GameSandboxDirectoryName;
}

FString FSandboxPlatformFile::ConvertToSandboxPath( const TCHAR* Filename ) const
{
	// Mostly for the malloc profiler to flush the data.
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FSandboxPlatformFile::ConvertToSandboxPath"), STAT_SandboxPlatformFile_ConvertToSandboxPath, STATGROUP_LoadTimeVerbose);

	// convert to a standardized path (relative)
	FString SandboxPath = Filename;
	FPaths::MakeStandardFilename(SandboxPath);

	if ((bSandboxEnabled == true) && (SandboxDirectory.Len() > 0))
	{
		if (InjectedTargetDirectory.Len())
		{
			// any path into the injected target will map to where we injected from (anyhing under the target path will StartsWith the injection target)
			if (SandboxPath.StartsWith(InjectedSourceDirectory))
			{
				// replace the start with InjectedTargetDirectory
				FString NewPath = InjectedTargetDirectory + SandboxPath.Mid(InjectedSourceDirectory.Len());
				UE_LOGF(LogInit, Verbose, "Injected %ls -->  %ls", *SandboxPath, *NewPath);

				// if (LowerLevel->DirectoryExists(*FPaths::GetPath(InjectedTargetDirectory)))
				{
					return NewPath;
				}
			}
			// for injection sandbox, we don't want to use it for anything outside of the injection point, so just return the original name
			return Filename;
		}

		// See whether Filename is relative to root directory.
		// if it's not inside the root, then just use it
		FString FullSandboxPath = FPaths::ConvertRelativePathToFull(SandboxPath);
		FString FullGameDir, FullSandboxedGameDir;
#if IS_PROGRAM
		if (FPaths::IsProjectFilePathSet())
		{
			FullGameDir = FPaths::ConvertRelativePathToFull(FPaths::GetPath(FPaths::GetProjectFilePath()) + TEXT("/"));
			FullSandboxedGameDir = FPaths::Combine(*SandboxDirectory, *FPaths::GetBaseFilename(FPaths::GetProjectFilePath()));
		}
		else
#endif
		{
			FullGameDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
			FullSandboxedGameDir = FPaths::Combine(*SandboxDirectory, FApp::GetProjectName());
		}
		if(FullSandboxPath.StartsWith(FullSandboxedGameDir))
		{
			return SandboxPath;
		}
		else if (FullSandboxPath.StartsWith(FullGameDir))
		{
#if IS_PROGRAM
			SandboxPath = FPaths::Combine(*SandboxDirectory, *FPaths::GetBaseFilename(FPaths::GetProjectFilePath()), *FullSandboxPath + FullGameDir.Len());
#else
			SandboxPath = FPaths::Combine(*SandboxDirectory, FApp::GetProjectName(), *FullSandboxPath + FullGameDir.Len());
#endif
		}
		else if (FullSandboxPath.StartsWith(*AbsoluteRootDirectory))
		{
			SandboxPath = FPaths::Combine(*SandboxDirectory, *FullSandboxPath + AbsoluteRootDirectory.Len());
		}
		else
		{
			int32 SeparatorIndex = SandboxPath.Find(TEXT("/"), ESearchCase::CaseSensitive);
			int32 SeparatorIndex2 = SandboxPath.Find(TEXT("\\"), ESearchCase::CaseSensitive);
			if (SeparatorIndex < 0 || (SeparatorIndex2 >= 0 && SeparatorIndex2 < SeparatorIndex))
			{
				SeparatorIndex = SeparatorIndex2;
			}
			FString DrivePath = SandboxPath;
			if (SeparatorIndex != INDEX_NONE)
			{
				DrivePath = SandboxPath.Mid( 0, SeparatorIndex );
			}
			if( FPaths::IsDrive( DrivePath ) == false )
			{
				FString Dir = FPlatformProcess::BaseDir();
				FPaths::MakeStandardFilename(Dir);
				SandboxPath = Dir / SandboxPath;
				SandboxPath = SandboxPath.Replace( *RelativeRootDirectory, *SandboxDirectory, ESearchCase::IgnoreCase );
			}
		}
	}

	return SandboxPath;
}

FString FSandboxPlatformFile::ConvertFromSandboxPath(const TCHAR* Filename) const
{
	FString FullSandboxPath = FPaths::ConvertRelativePathToFull(Filename);

	FString SandboxGameDirectory = FPaths::Combine(*SandboxDirectory, FApp::GetProjectName());
	FString SandboxRootDirectory = SandboxDirectory;

	FString OriginalPath;

	if (FullSandboxPath.StartsWith(SandboxGameDirectory))
	{
		OriginalPath = FullSandboxPath.Replace(*SandboxGameDirectory, *FPaths::ProjectDir());
	}
	else if (FullSandboxPath.StartsWith(SandboxRootDirectory))
	{
		OriginalPath = FullSandboxPath.Replace(*SandboxRootDirectory, *FPaths::RootDir());
	}

	OriginalPath.ReplaceInline(TEXT("//"), TEXT("/"));

	FString FullOriginalPath = FPaths::ConvertRelativePathToFull(OriginalPath);

	return FullOriginalPath;
}

bool FSandboxPlatformFile::WipeSandboxFolder( const TCHAR* AbsolutePath )
{
	return DeleteDirectory( AbsolutePath, true );
}

bool FSandboxPlatformFile::DeleteDirectory( const TCHAR* Path, bool Tree )
{
	if( Tree )
	{
		// Support code for removing a directory tree.
		bool Result = true;
		// Delete all files in the directory first
		FString Spec = FString( Path ) / TEXT( "*" );
		TArray<FString> List;
		FindFiles( List, *Spec, true, false );
		for( int32 FileIndex = 0; FileIndex < List.Num(); FileIndex++ )
		{
			FString Filename( FString( Path ) / List[ FileIndex ] );
			// Delete the file even if it's read-only
			if( LowerLevel->FileExists( *Filename ) )
			{
				LowerLevel->SetReadOnly( *Filename, false );
				if( !LowerLevel->DeleteFile( *Filename ) )
				{
					Result = false;
				}				
			}
			else
			{
				Result = false;
			}
		}
		// Clear out the list of found files and look for directories this time
		List.Empty();
		FindFiles( List, *Spec, false, true );
		for( int32 DirectoryIndex = 0; DirectoryIndex < List.Num(); DirectoryIndex++ )
		{
			if( !DeleteDirectory( *( FString( Path ) / List[ DirectoryIndex ] ), true ) )
			{
				Result = false;
			}
		}
		// The directory is empty now so it can be deleted
		return DeleteDirectory( Path, false ) && Result;
	}
	else
	{
		return LowerLevel->DeleteDirectory( Path ) || ( !LowerLevel->DirectoryExists( Path ) );
	}
}

void FSandboxPlatformFile::FindFiles( TArray<FString>& Result, const TCHAR* InFilename, bool Files, bool Directories )
{
	class FFileMatch : public IPlatformFile::FDirectoryVisitor
	{
	public:
		TArray<FString>& Result;
		FString WildCard;
		bool bFiles;
		bool bDirectories;
		FFileMatch( TArray<FString>& InResult, const FString& InWildCard, bool bInFiles, bool bInDirectories )
			: Result( InResult )
			, WildCard( InWildCard )
			, bFiles( bInFiles )
			, bDirectories( bInDirectories )
		{
		}
		virtual bool Visit( const TCHAR* FilenameOrDirectory, bool bIsDirectory )
		{
			if( ( bIsDirectory && bDirectories ) ||
				( !bIsDirectory && bFiles && FString( FilenameOrDirectory ).MatchesWildcard( WildCard ) ) )
			{
				Result.Add( FPaths::GetCleanFilename( FilenameOrDirectory ) );
			}
			return true;
		}
	};
	
	FFileMatch FileMatch( Result, FPaths::GetCleanFilename(InFilename), Files, Directories );
	LowerLevel->IterateDirectory( *FPaths::GetPath(InFilename), FileMatch );
}

FString FSandboxPlatformFile::ConvertToAbsolutePathForExternalAppForRead( const TCHAR* Filename )
{
	FString SandboxPath( *ConvertToSandboxPath( Filename ) );
	if ( LowerLevel->FileExists( *SandboxPath ) || !OkForInnerAccess(Filename))
	{
		return SandboxPath;
	}
	else
	{
		return FPaths::ConvertRelativePathToFull(Filename);
	}
}

FString FSandboxPlatformFile::ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* Filename )
{
	return FPaths::ConvertRelativePathToFull(ConvertToSandboxPath(Filename));
}

const FString& FSandboxPlatformFile::GetAbsolutePathToGameDirectory()
{
	if (AbsolutePathToGameDirectory.IsEmpty())
	{
		// Strip game directory, keep just to path to the game directory which could simply be the root dir (but not always).
		AbsolutePathToGameDirectory = FPaths::GetPath(GetAbsoluteGameDirectory());
	}
	return AbsolutePathToGameDirectory;
}

const FString& FSandboxPlatformFile::GetAbsoluteGameDirectory()
{
	if (AbsoluteGameDirectory.IsEmpty())
	{
		AbsoluteGameDirectory = FPaths::ProjectDir();
		UE_CLOGF(AbsoluteGameDirectory.IsEmpty(), SandboxFile, Fatal, "SandboxFileWrapper tried to access project path before it was set.");
		AbsoluteGameDirectory = FPaths::ConvertRelativePathToFull(AbsoluteGameDirectory);
		// Strip .uproject filename
		AbsoluteGameDirectory = FPaths::GetPath(AbsoluteGameDirectory);
	}
	return AbsoluteGameDirectory;
}


bool FSandboxPlatformFile::OkForInnerAccess(const TCHAR* InFilenameOrDirectoryName, bool bIsDirectory) const
{
	if (DirectoryExclusionWildcards.Num() || FileExclusionWildcards.Num())
	{
		FString FilenameOrDirectoryName(InFilenameOrDirectoryName);
		FPaths::MakeStandardFilename(FilenameOrDirectoryName);
		if (bIsDirectory)
		{
			for (int32 Index = 0; Index < DirectoryExclusionWildcards.Num(); Index++)
			{
				if (FilenameOrDirectoryName.MatchesWildcard(DirectoryExclusionWildcards[Index]))
				{
					return false;
				}
			}
		}
		else
		{
			for (int32 Index = 0; Index < FileExclusionWildcards.Num(); Index++)
			{
				if (FilenameOrDirectoryName.MatchesWildcard(FileExclusionWildcards[Index]))
				{
					return false;
				}
			}
		}
	}
	return !bSandboxOnly;
}

void FSandboxPlatformFile::SetSandboxEnabled(bool bInEnabled)
{
	bSandboxEnabled = bInEnabled;
}

bool FSandboxPlatformFile::IsSandboxEnabled() const
{
	return bSandboxEnabled;
}

IPlatformFile* FSandboxPlatformFile::GetLowerLevel()
{
	return LowerLevel;
}

void FSandboxPlatformFile::SetLowerLevel(IPlatformFile* NewLowerLevel)
{
	LowerLevel = NewLowerLevel;
}

const TCHAR* FSandboxPlatformFile::GetName() const
{
	return FSandboxPlatformFile::GetTypeName();
}

void FSandboxPlatformFile::AddExclusion(const TCHAR* Wildcard, bool bIsDirectory)
{
	if (bIsDirectory)
	{
		DirectoryExclusionWildcards.AddUnique(FString(Wildcard));
	}
	else
	{
		FileExclusionWildcards.AddUnique(FString(Wildcard));
	}
}

void FSandboxPlatformFile::RemoveExclusion(const TCHAR* Wildcard, bool bIsDirectory)
{
	if (bIsDirectory)
	{
		DirectoryExclusionWildcards.Remove(FString(Wildcard));
	}
	else
	{
		FileExclusionWildcards.Remove(FString(Wildcard));
	}
}

void FSandboxPlatformFile::SetSandboxOnly(bool bInSandboxOnly)
{
	bSandboxOnly = bInSandboxOnly;
}

// IPlatformFile Interface

bool FSandboxPlatformFile::FileExists(const TCHAR* Filename)
{
	// First look for the file in the user dir.
	bool Result = LowerLevel->FileExists(*ConvertToSandboxPath(Filename));
	if (Result == false && OkForInnerAccess(Filename))
	{
		Result = LowerLevel->FileExists(Filename);
	}
	return Result;
}

int64 FSandboxPlatformFile::FileSize(const TCHAR* Filename)
{
	// First look for the file in the user dir.
	int64 Result = LowerLevel->FileSize(*ConvertToSandboxPath(Filename));
	if (Result < 0 && OkForInnerAccess(Filename))
	{
		Result = LowerLevel->FileSize(Filename);
	}
	return Result;
}

bool FSandboxPlatformFile::DeleteFile(const TCHAR* Filename)
{
	// Delete only sandbox files. If the sendbox version doesn't exists
	// assume the delete was successful because we only care if the sandbox version is gone.
	bool Result = true;
	FString SandboxFilename(*ConvertToSandboxPath(Filename));
	if (LowerLevel->FileExists(*SandboxFilename))
	{
		Result = LowerLevel->DeleteFile(*ConvertToSandboxPath(Filename));
	}
	return Result;
}

bool FSandboxPlatformFile::IsReadOnly(const TCHAR* Filename)
{
	// If the file exists in the sandbox folder and is read-only return true
	// Otherwise it can always be 'overwritten' in the sandbox
	bool Result = false;
	FString SandboxFilename(*ConvertToSandboxPath(Filename));
	if (LowerLevel->FileExists(*SandboxFilename))
	{
		// If the file exists in sandbox dir check its read-only flag
		Result = LowerLevel->IsReadOnly(*SandboxFilename);
	}
	//else
	//{
	//	// Fall back to normal directory
	//	Result = LowerLevel->IsReadOnly( Filename );
	//}
	return Result;
}

bool FSandboxPlatformFile::MoveFile(const TCHAR* To, const TCHAR* From)
{
	// Only files within the sandbox dir can be moved
	bool Result = false;
	FString SandboxFilename(*ConvertToSandboxPath(From));
	if (LowerLevel->FileExists(*SandboxFilename) || LowerLevel->DirectoryExists(*SandboxFilename))
	{
		Result = LowerLevel->MoveFile(*ConvertToSandboxPath(To), *SandboxFilename);
	}
	return Result;
}

bool FSandboxPlatformFile::SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue)
{
	bool Result = false;
	FString UserFilename(*ConvertToSandboxPath(Filename));
	if (LowerLevel->FileExists(*UserFilename))
	{
		Result = LowerLevel->SetReadOnly(*UserFilename, bNewReadOnlyValue);
	}
	return Result;
}

FDateTime FSandboxPlatformFile::GetTimeStamp(const TCHAR* Filename)
{
	FDateTime Result = FDateTime::MinValue();
	FString UserFilename(*ConvertToSandboxPath(Filename));
	Result = LowerLevel->GetTimeStamp(*UserFilename);
	if ((Result == FDateTime::MinValue()) && OkForInnerAccess(Filename))
	{
		Result = LowerLevel->GetTimeStamp(Filename);
	}
	return Result;
}

void FSandboxPlatformFile::SetTimeStamp(const TCHAR* Filename, FDateTime DateTime)
{
	FString UserFilename(*ConvertToSandboxPath(Filename));
	if (LowerLevel->FileExists(*UserFilename))
	{
		LowerLevel->SetTimeStamp(*UserFilename, DateTime);
	}
	else if (OkForInnerAccess(Filename))
	{
		LowerLevel->SetTimeStamp(Filename, DateTime);
	}
}

FDateTime FSandboxPlatformFile::GetAccessTimeStamp(const TCHAR* Filename)
{
	FDateTime Result = FDateTime::MinValue();
	FString UserFilename(*ConvertToSandboxPath(Filename));
	if (LowerLevel->FileExists(*UserFilename))
	{
		Result = LowerLevel->GetAccessTimeStamp(*UserFilename);
	}
	else if (OkForInnerAccess(Filename))
	{
		Result = LowerLevel->GetAccessTimeStamp(Filename);
	}
	return Result;
}

FString	FSandboxPlatformFile::GetFilenameOnDisk(const TCHAR* Filename)
{
	FString Result;
	FString UserFilename(*ConvertToSandboxPath(Filename));
	if (LowerLevel->FileExists(*UserFilename))
	{
		Result = LowerLevel->GetFilenameOnDisk(*UserFilename);
	}
	else if (OkForInnerAccess(Filename))
	{
		Result = LowerLevel->GetFilenameOnDisk(Filename);
	}
	return Result;
}

IFileHandle* FSandboxPlatformFile::OpenRead(const TCHAR* Filename, bool bAllowWrite)
{
	IFileHandle* Result = LowerLevel->OpenRead(*ConvertToSandboxPath(Filename), bAllowWrite);
	if (!Result && OkForInnerAccess(Filename))
	{
		Result = LowerLevel->OpenRead(Filename);
	}
	return Result;
}

IFileHandle* FSandboxPlatformFile::OpenWrite(const TCHAR* Filename, bool bAppend, bool bAllowRead)
{
	// Only files from the sandbox directory can be opened for wiriting
	return LowerLevel->OpenWrite(*ConvertToSandboxPath(Filename), bAppend, bAllowRead);
}

bool FSandboxPlatformFile::DirectoryExists(const TCHAR* Directory)
{
	bool Result = LowerLevel->DirectoryExists(*ConvertToSandboxPath(Directory));
	if (Result == false && OkForInnerAccess(Directory, true))
	{
		Result = LowerLevel->DirectoryExists(Directory);
	}
	return Result;
}

bool FSandboxPlatformFile::CreateDirectory(const TCHAR* Directory)
{
	// Directories can be created only under the sandbox path
	return LowerLevel->CreateDirectory(*ConvertToSandboxPath(Directory));
}

bool FSandboxPlatformFile::DeleteDirectory(const TCHAR* Directory)
{
	// Directories can be deleted only under the sandbox path
	return LowerLevel->DeleteDirectory(*ConvertToSandboxPath(Directory));
}

FFileStatData FSandboxPlatformFile::GetStatData(const TCHAR* FilenameOrDirectory)
{
	FFileStatData Result = LowerLevel->GetStatData(*ConvertToSandboxPath(FilenameOrDirectory));
	if (!Result.bIsValid && (OkForInnerAccess(FilenameOrDirectory, false) && OkForInnerAccess(FilenameOrDirectory, true)))
	{
		Result = LowerLevel->GetStatData(FilenameOrDirectory);
	}
	return Result;
}

class FSandboxVisitor : public IPlatformFile::FDirectoryVisitor
{
public:
	FDirectoryVisitor& Visitor;
	FSandboxPlatformFile& SandboxFile;
	TSet<FString> VisitedSandboxFiles;
	bool bIsRecursive = false;
	bool bKeepIterating = true;

	FSandboxVisitor(FDirectoryVisitor& InVisitor, FSandboxPlatformFile& InSandboxFile, bool bInIsRecursive)
		: Visitor(InVisitor)
		, SandboxFile(InSandboxFile)
		, bIsRecursive(bInIsRecursive)
	{
	}
	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		bool CanVisit = true;
		FString LocalFilename(FilenameOrDirectory);

		bool bHasTranslated = false;
		if (SandboxFile.InjectedTargetDirectory.Len())
		{
			// if we visit the parent of what we are injecting, we need to force visit the injected directory - because it doesn't exist in the original
			// location, we will never visit it to convert it to the injected location

			// convert to standard filename for matching (like ConvertToSandboxPath does)
			FString NormalizedFilename(LocalFilename);
			FPaths::MakeStandardFilename(NormalizedFilename);

			if (NormalizedFilename == SandboxFile.InjectedSourceDirectoryParent)
			{
				// "fake" the injected directory so everything falls into place
				bKeepIterating = bKeepIterating && Visitor.CallShouldVisitAndVisit(*SandboxFile.InjectedSourceDirectory, true);
				if (!bKeepIterating)
				{
					return false;
				}

				// for recursive visiting, we need to recurse into the forced directory
				if (bIsRecursive)
				{
					SandboxFile.LowerLevel->IterateDirectoryRecursively(*SandboxFile.InjectedTargetDirectory, *this);
					if (!bKeepIterating)
					{
						return false;
					}
				}
			}

			if (LocalFilename.StartsWith(SandboxFile.InjectedTargetDirectory))
			{
				LocalFilename = SandboxFile.InjectedSourceDirectory + LocalFilename.Mid(SandboxFile.InjectedTargetDirectory.Len());
				bHasTranslated = true;
			}
		}


		if (!bHasTranslated)
		{
			if (FCString::Strnicmp(*LocalFilename, *SandboxFile.GetSandboxDirectory(), SandboxFile.GetSandboxDirectory().Len()) == 0)
			{
				// FilenameOrDirectory is already pointing to the sandbox directory so add it to the list of sanbox files.
				// The filename is always stored with the abslute sandbox path.
				VisitedSandboxFiles.Add(*LocalFilename);
				// Now convert the sandbox path back to engine path because the sandbox folder should not be exposed
				// to the engine and remain transparent.
				LocalFilename.MidInline(SandboxFile.GetSandboxDirectory().Len(), MAX_int32, EAllowShrinking::No);
				if (LocalFilename.StartsWith(TEXT("Engine/")) || (FCString::Stricmp(*LocalFilename, TEXT("Engine")) == 0))
				{
					LocalFilename = SandboxFile.GetAbsoluteRootDirectory() / LocalFilename;
				}
				else
				{
					LocalFilename.MidInline(SandboxFile.GetGameSandboxDirectoryName().Len(), MAX_int32, EAllowShrinking::No);
					LocalFilename = SandboxFile.GetAbsoluteGameDirectory() / LocalFilename;
				}
			}
			else
			{
				// Favourize Sandbox files over normal path files.
				CanVisit = !VisitedSandboxFiles.Contains(SandboxFile.ConvertToSandboxPath(*LocalFilename))
					&& SandboxFile.OkForInnerAccess(*LocalFilename, bIsDirectory);
			}
		}

		if (CanVisit)
		{
			bKeepIterating = bKeepIterating && Visitor.CallShouldVisitAndVisit(*LocalFilename, bIsDirectory);
			if (!bKeepIterating)
			{
				return false;
			}
		}
		// Continue iterating.
		return true;
	}
};

bool FSandboxPlatformFile::IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor)
{
	FSandboxVisitor SandboxVisitor(Visitor, *this, false);
	FString SandboxDir = ConvertToSandboxPath(Directory);
	bool Result = LowerLevel->IterateDirectory(*SandboxDir, SandboxVisitor);
	// don't iterate the same directory twice if the Convert didn't change the path, or if we were asked to stop iterating
	if (SandboxVisitor.bKeepIterating && (SandboxDir != Directory))
	{
		Result = LowerLevel->IterateDirectory(Directory, SandboxVisitor) || Result;
	}
	return Result && SandboxVisitor.bKeepIterating;
}

bool FSandboxPlatformFile::IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor)
{
	FSandboxVisitor SandboxVisitor(Visitor, *this, true);
	FString SandboxDir = ConvertToSandboxPath(Directory);
	bool Result = LowerLevel->IterateDirectoryRecursively(*SandboxDir, SandboxVisitor);
	// don't iterate the same directory twice if the Convert didn't change the path, or if we were asked to stop iterating
	if (SandboxVisitor.bKeepIterating && (SandboxDir != Directory))
	{
		Result = LowerLevel->IterateDirectoryRecursively(Directory, SandboxVisitor) || Result;
	}
	return Result && SandboxVisitor.bKeepIterating;
}

class FSandboxStatVisitor : public IPlatformFile::FDirectoryStatVisitor
{
public:
	FDirectoryStatVisitor& Visitor;
	FSandboxPlatformFile& SandboxFile;
	TSet<FString> VisitedSandboxFiles;
	bool bIsRecursive = false;
	bool bKeepIterating = true;

	FSandboxStatVisitor(FDirectoryStatVisitor& InVisitor, FSandboxPlatformFile& InSandboxFile, bool bInIsRecursive)
		: Visitor(InVisitor)
		, SandboxFile(InSandboxFile)
		, bIsRecursive(bInIsRecursive)
	{
	}
	virtual bool Visit(const TCHAR* FilenameOrDirectory, const FFileStatData& StatData) override
	{
		bool CanVisit = true;
		FString LocalFilename(FilenameOrDirectory);

		bool bHasTranslated = false;
		if (SandboxFile.InjectedTargetDirectory.Len())
		{
			// if we visit the parent of what we are injecting, we need to force visit the injected directory - because it doesn't exist in the original
			// location, we will never visit it to convert it to the injected location

			// convert to standard filename for matching (like ConvertToSandboxPath does)
			FString NormalizedFilename(LocalFilename);
			FPaths::MakeStandardFilename(NormalizedFilename);

			if (NormalizedFilename == SandboxFile.InjectedSourceDirectoryParent)
			{
				// "fake" the injected directory so everything falls into place
				FFileStatData InjectedStat = SandboxFile.LowerLevel->GetStatData(*SandboxFile.InjectedSourceDirectory);
				bKeepIterating = bKeepIterating && Visitor.CallShouldVisitAndVisit(*SandboxFile.InjectedSourceDirectory, InjectedStat);
				if (!bKeepIterating)
				{
					return false;
				}

				// for recursive visiting, we need to recurse into the forced directory
				if (bIsRecursive)
				{
					SandboxFile.LowerLevel->IterateDirectoryStatRecursively(*SandboxFile.InjectedTargetDirectory, *this);
					if (!bKeepIterating)
					{
						return false;
					}
				}
			}

			if (LocalFilename.StartsWith(SandboxFile.InjectedTargetDirectory))
			{
				LocalFilename = SandboxFile.InjectedSourceDirectory + LocalFilename.Mid(SandboxFile.InjectedTargetDirectory.Len());
				bHasTranslated = true;
			}
		}

		if (!bHasTranslated)
		{
			if (FCString::Strnicmp(*LocalFilename, *SandboxFile.GetSandboxDirectory(), SandboxFile.GetSandboxDirectory().Len()) == 0)
			{
				// FilenameOrDirectory is already pointing to the sandbox directory so add it to the list of sanbox files.
				// The filename is always stored with the abslute sandbox path.
				VisitedSandboxFiles.Add(*LocalFilename);
				// Now convert the sandbox path back to engine path because the sandbox folder should not be exposed
				// to the engine and remain transparent.
				LocalFilename.MidInline(SandboxFile.GetSandboxDirectory().Len(), MAX_int32, EAllowShrinking::No);
				if (LocalFilename.StartsWith(TEXT("Engine/")))
				{
					LocalFilename = SandboxFile.GetAbsoluteRootDirectory() / LocalFilename;
				}
				else
				{
					LocalFilename = SandboxFile.GetAbsolutePathToGameDirectory() / LocalFilename;
				}
			}
			else
			{
				// Favourize Sandbox files over normal path files.
				CanVisit = !VisitedSandboxFiles.Contains(SandboxFile.ConvertToSandboxPath(*LocalFilename))
					&& SandboxFile.OkForInnerAccess(*LocalFilename, StatData.bIsDirectory);
			}
		}

		if (CanVisit)
		{
			bKeepIterating = bKeepIterating && Visitor.CallShouldVisitAndVisit(*LocalFilename, StatData);
			if (!bKeepIterating)
			{
				return false;
			}
		}
		// Continue iterating.
		return true;
	}
};

bool		FSandboxPlatformFile::IterateDirectoryStat(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor)
{
	FSandboxStatVisitor SandboxVisitor(Visitor, *this, false);
	FString SandboxDir = ConvertToSandboxPath(Directory);
	bool Result = LowerLevel->IterateDirectoryStat(*SandboxDir, SandboxVisitor);
	// don't iterate the same directory twice if the Convert didn't change the path, or if we were asked to stop iterating (which can happen with injection type)
	if (SandboxVisitor.bKeepIterating && (SandboxDir != Directory))
	{
		Result = LowerLevel->IterateDirectoryStat(Directory, SandboxVisitor) || Result;
	}
	return Result && SandboxVisitor.bKeepIterating;
}

bool		FSandboxPlatformFile::IterateDirectoryStatRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor)
{
	FSandboxStatVisitor SandboxVisitor(Visitor, *this, true);
	FString SandboxDir = ConvertToSandboxPath(Directory);
	bool Result = LowerLevel->IterateDirectoryStatRecursively(*ConvertToSandboxPath(Directory), SandboxVisitor);
	// don't iterate the same directory twice if the Convert didn't change the path, or if we were asked to stop iterating (which can happen with injection type)
	if (SandboxVisitor.bKeepIterating && (SandboxDir != Directory))
	{
		Result = LowerLevel->IterateDirectoryStatRecursively(Directory, SandboxVisitor) || Result;
	}
	return Result && SandboxVisitor.bKeepIterating;
}

bool		FSandboxPlatformFile::DeleteDirectoryRecursively(const TCHAR* Directory)
{
	// Directories can be deleted only under the sandbox path
	return LowerLevel->DeleteDirectoryRecursively(*ConvertToSandboxPath(Directory));
}

bool FSandboxPlatformFile::CreateDirectoryTree(const TCHAR* Directory)
{
	// Directories can only be created only under the sandbox path
	return LowerLevel->CreateDirectoryTree(*ConvertToSandboxPath(Directory));
}

bool FSandboxPlatformFile::CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags, EPlatformFileWrite WriteFlags)
{
	// Files can be copied only to the sandbox directory
	bool Result = false;
	if (LowerLevel->FileExists(*ConvertToSandboxPath(From)))
	{
		Result = LowerLevel->CopyFile(*ConvertToSandboxPath(To), *ConvertToSandboxPath(From), ReadFlags, WriteFlags);
	}
	else
	{
		Result = LowerLevel->CopyFile(*ConvertToSandboxPath(To), From, ReadFlags, WriteFlags);
	}
	return Result;
}

IAsyncReadFileHandle* FSandboxPlatformFile::OpenAsyncRead(const TCHAR* Filename, bool bAllowWrite /*= false*/)
{
	FString UserFilename(*ConvertToSandboxPath(Filename));
	if (!OkForInnerAccess(Filename) || LowerLevel->FileExists(*UserFilename))
	{
		return LowerLevel->OpenAsyncRead(*UserFilename, bAllowWrite);
	}
	return LowerLevel->OpenAsyncRead(Filename, bAllowWrite);
}

void FSandboxPlatformFile::SetAsyncMinimumPriority(EAsyncIOPriorityAndFlags Priority)
{
	LowerLevel->SetAsyncMinimumPriority(Priority);
}

FOpenMappedResult FSandboxPlatformFile::OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions, int64 MaximumSize)
{
	FString UserFilename(*ConvertToSandboxPath(Filename));
	if (!OkForInnerAccess(Filename) || LowerLevel->FileExists(*UserFilename))
	{
		return LowerLevel->OpenMappedEx(*UserFilename, OpenOptions, MaximumSize);
	}
	return LowerLevel->OpenMappedEx(Filename, OpenOptions, MaximumSize);
}

FOpenMappedResult FSandboxPlatformFile::OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions, int64 MaximumSize)
{
	FString UserFilename(*ConvertToSandboxPath(Filename));
	if (!OkForInnerAccess(Filename) || LowerLevel->FileExists(*UserFilename))
	{
		return LowerLevel->OpenMappedEx2(*UserFilename, OpenOptions, MaximumSize);
	}
	return LowerLevel->OpenMappedEx2(Filename, OpenOptions, MaximumSize);
}

/**
 * Module for the sandbox file
 */
class FSandboxFileModule : public IPlatformFileModule
{
public:
	virtual IPlatformFile* GetPlatformFile() override
	{
		static TUniquePtr<IPlatformFile> AutoDestroySingleton = FSandboxPlatformFile::Create(/* bInEntireEngineWillUseThisSandbox */ true);
		return AutoDestroySingleton.Get();
	}
};
IMPLEMENT_MODULE(FSandboxFileModule, SandboxFile);
