// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "Misc/DateTime.h"
#include "Misc/Paths.h"
#include "Templates/UniquePtr.h"

class IAsyncReadFileHandle;
class IMappedFileHandle;

/**
 * Wrapper to log the low level file system
**/
DECLARE_LOG_CATEGORY_EXTERN(SandboxFile, Log, All);

class FSandboxPlatformFile : public IPlatformFile
{
	/** Wrapped file */
	IPlatformFile*		LowerLevel;
	/** Absolute path to the sandbox directory */
	FString			SandboxDirectory;
	/** Name of the game's sandbox directory */
	FString			GameSandboxDirectoryName;
	/** Relative path to root directory. Cached for faster access */
	FString			RelativeRootDirectory;
	/** Absolute path to root directory. Cached for faster access */
	FString			AbsoluteRootDirectory;
	/** Absolute game directory. Cached for faster access */
	FString			AbsoluteGameDirectory;
	/** Absolute path to game directory. Cached for faster access */
	FString			AbsolutePathToGameDirectory;
	/** Access to any file (in unreal standard form) matching this is not allowed */
	TArray<FString>		FileExclusionWildcards;
	/** Access to any directory (in unreal standard form) matching this is not allowed */
	TArray<FString>		DirectoryExclusionWildcards;
	bool				bEntireEngineWillUseThisSandbox;

	/**
	 *	Whether the sandbox is enabled or not.
	 *	Defaults to true.
	 *	Set to false when operations require writing to the actual physical location given.
	 */
	bool				bSandboxEnabled;

	/**
	 * For an injection sandbox, we insert the contents of one directory into the contents of another directory (InjectedSource)
	 * in the eyes of the engine (InjectedTarget). So you could inject C:\MyPlugins\Foo into ../../../MyProject/Plugins/Foo
	 * and as far as the engine knows, Foo is underneath your MyProject directory
	 */
	FString				InjectedSourceDirectory;
	FString				InjectedSourceDirectoryParent;
	FString				InjectedTargetDirectory;
	FString				InjectedTargetDirectoryParent;

	/**
	 *	Whether access is restricted to the sandbox or not.
	 *	Defaults to false.
	 */
	bool				bSandboxOnly;

	/**
	 * Clears the contents of the specified folder
	 *
	 * @param AbsolutePath Absolute path to the folder to wipe
	 * @return true if the folder's contents could be deleted, false otherwise
	 */
	SANDBOXFILE_API bool WipeSandboxFolder( const TCHAR* AbsolutePath );	

	/**
	 * Finds all files or folders in the given directory.
	 * This is partially copied from file manager but since IPlatformFile is lower level
	 * it's better to have local version which doesn't use the wrapped IPlatformFile.
	 *
	 * @param Result List of found files or folders.
	 * @param InFilename Path to the folder to look in.
	 * @param Files true to include files in the Result
	 * @param Files true to include directories in the Result
	 */
	SANDBOXFILE_API void FindFiles( TArray<FString>& Result, const TCHAR* InFilename, bool Files, bool Directories );
	
	/** Allow IPlatformFile::FindFiles */
	using IPlatformFile::FindFiles;

	/**
	 * Deletes a directory
	 * This is partially copied from file manager but since IPlatformFile is lower level
	 * it's better to have local version which doesn't use the wrapped IPlatformFile.
	 *
	 * @param Path Path to the directory to delete.
	 * @param Tree true to recursively delete the directory and its contents
	 * @return true if the operaton was successful.
	 */
	SANDBOXFILE_API bool DeleteDirectory( const TCHAR* Path, bool Tree );

	/**
	 * Check if a file or directory has been filtered, and hence is unavailable to the outside world
	 *
	 * @param FilenameOrDirectoryName 
	 * @param bIsDirectory if true, this is a directory
	 * @return true if it is ok to access the non-sandboxed files here
	 */
	SANDBOXFILE_API bool OkForInnerAccess(const TCHAR* InFilenameOrDirectoryName, bool bIsDirectory = false) const;

	static const TCHAR* GetTypeName()
	{
		return TEXT("SandboxFile");
	}

	/**
	 * Converts passed in filename to use a sandbox path.
	 * @param	bEntireEngineWillUseThisSandbox		If true, the we set up the engine so that subprocesses also use this subdirectory
	 */
	SANDBOXFILE_API FSandboxPlatformFile(bool bInEntireEngineWillUseThisSandbox);

public:
	static SANDBOXFILE_API TUniquePtr<FSandboxPlatformFile> Create(bool bInEntireEngineWillUseThisSandbox);

	SANDBOXFILE_API virtual ~FSandboxPlatformFile();

	//~ For visibility of overloads we don't override
	using IPlatformFile::IterateDirectory;
	using IPlatformFile::IterateDirectoryRecursively;
	using IPlatformFile::IterateDirectoryStat;
	using IPlatformFile::IterateDirectoryStatRecursively;

	/**
	 *	Set whether the sandbox is enabled or not
	 *
	 *	@param	bInEnabled		true to enable the sandbox, false to disable it
	 */
	SANDBOXFILE_API virtual void SetSandboxEnabled(bool bInEnabled) override;

	/**
	 *	Returns whether the sandbox is enabled or not
	 *
	 *	@return	bool			true if enabled, false if not
	 */
	SANDBOXFILE_API virtual bool			IsSandboxEnabled() const override;

	SANDBOXFILE_API virtual bool			ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const override;
	SANDBOXFILE_API virtual bool			Initialize(IPlatformFile* Inner, const TCHAR* CmdLine) override;

	SANDBOXFILE_API virtual IPlatformFile*	GetLowerLevel() override;
	SANDBOXFILE_API virtual void			SetLowerLevel(IPlatformFile* NewLowerLevel) override;
	SANDBOXFILE_API virtual const TCHAR*	GetName() const override;

	/**
	 * Converts passed in filename to use a sandbox path.
	 *
	 * @param	Filename	filename (under game directory) to convert to use a sandbox path. Can be relative or absolute.
	 * 
	 * @return	filename using sandbox path
	 */
	SANDBOXFILE_API FString ConvertToSandboxPath(const TCHAR* Filename) const;
	SANDBOXFILE_API FString ConvertFromSandboxPath(const TCHAR* Filename) const;

	/** Returns sandbox directory */
	SANDBOXFILE_API const FString& GetSandboxDirectory() const;

	/** Returns the name of the sandbox directory for the game's content */
	SANDBOXFILE_API const FString& GetGameSandboxDirectoryName();

	/** Returns absolute root directory */
	SANDBOXFILE_API const FString& GetAbsoluteRootDirectory() const;

	/** Returns absolute game directory */
	SANDBOXFILE_API const FString& GetAbsoluteGameDirectory();

	/** Returns absolute path to game directory (without the game directory itself) */
	SANDBOXFILE_API const FString& GetAbsolutePathToGameDirectory();

	/** 
	 * Add exclusion. These files and / or directories pretend not to exist so that they cannot be accessed at all (except in the sandbox) 
	 * @param Wildcard FString::MatchesWildcard-type wild card to test for exclusion
	 * @param bIsDirectory if true, this is a directory
	 * @Caution, these have a performance cost
	*/
	SANDBOXFILE_API void AddExclusion(const TCHAR* Wildcard, bool bIsDirectory = false);
	SANDBOXFILE_API void RemoveExclusion(const TCHAR* Wildcard, bool bIsDirectory = false);

	/** Whether access is restricted the the sandbox or not. */
	SANDBOXFILE_API void SetSandboxOnly(bool bInSandboxOnly);

	// IPlatformFile Interface

	SANDBOXFILE_API virtual bool					FileExists(const TCHAR* Filename) override;
	SANDBOXFILE_API virtual int64					FileSize(const TCHAR* Filename) override;
	SANDBOXFILE_API virtual bool					DeleteFile(const TCHAR* Filename) override;
	SANDBOXFILE_API virtual bool					IsReadOnly(const TCHAR* Filename) override;
	SANDBOXFILE_API virtual bool					MoveFile(const TCHAR* To, const TCHAR* From) override;
	SANDBOXFILE_API virtual bool					SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override;
	SANDBOXFILE_API virtual FDateTime				GetTimeStamp(const TCHAR* Filename) override;
	SANDBOXFILE_API virtual void 					SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) override;
	SANDBOXFILE_API virtual FDateTime				GetAccessTimeStamp(const TCHAR* Filename) override;
	SANDBOXFILE_API virtual FString					GetFilenameOnDisk(const TCHAR* Filename) override;
	SANDBOXFILE_API virtual IFileHandle*			OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override;
	SANDBOXFILE_API virtual IFileHandle*			OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override;
	SANDBOXFILE_API virtual bool					DirectoryExists(const TCHAR* Directory) override;
	SANDBOXFILE_API virtual bool					CreateDirectory(const TCHAR* Directory) override;
	SANDBOXFILE_API virtual bool					DeleteDirectory(const TCHAR* Directory) override;
	SANDBOXFILE_API virtual FFileStatData			GetStatData(const TCHAR* FilenameOrDirectory) override;

	SANDBOXFILE_API virtual bool					IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override;
	SANDBOXFILE_API virtual bool					IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override;

	SANDBOXFILE_API virtual bool					IterateDirectoryStat(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) override;
	SANDBOXFILE_API virtual bool					IterateDirectoryStatRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) override;

	SANDBOXFILE_API virtual bool					DeleteDirectoryRecursively(const TCHAR* Directory) override;
	SANDBOXFILE_API virtual bool					CreateDirectoryTree(const TCHAR* Directory) override;

	SANDBOXFILE_API virtual bool					CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags = EPlatformFileRead::None, EPlatformFileWrite WriteFlags = EPlatformFileWrite::None) override;
	SANDBOXFILE_API virtual FString					ConvertToAbsolutePathForExternalAppForRead( const TCHAR* Filename ) override;
	SANDBOXFILE_API virtual FString					ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* Filename ) override;

	SANDBOXFILE_API virtual IAsyncReadFileHandle*	OpenAsyncRead(const TCHAR* Filename, bool bAllowWrite = false) override;
	SANDBOXFILE_API virtual void					SetAsyncMinimumPriority(EAsyncIOPriorityAndFlags Priority) override;

	SANDBOXFILE_API virtual FOpenMappedResult		OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions = EOpenReadFlags::None, int64 MaximumSize = 0) override;
	SANDBOXFILE_API virtual FOpenMappedResult		OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions = EOpenMappedFlags::None, int64 MaximumSize = 0) override;

	friend class FSandboxVisitor;
	friend class FSandboxStatVisitor;
};
