// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SandboxFile : ModuleRules
{
	public SandboxFile(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("Core");
	}
}
